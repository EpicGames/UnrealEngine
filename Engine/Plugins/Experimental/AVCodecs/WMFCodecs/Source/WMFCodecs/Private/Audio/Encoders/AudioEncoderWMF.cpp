// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/Encoders/AudioEncoderWMF.h"

THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include <mfapi.h>
#include <mftransform.h>
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

#include "SampleBuffer.h"

#include "Audio/Resources/AudioResourceCPU.h"

FAudioEncoderWMF::~FAudioEncoderWMF()
{
	Close();
}

bool FAudioEncoderWMF::IsOpen() const
{
	return bOpen;
}

FAVResult FAudioEncoderWMF::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();
	
	TAudioEncoder::Open(NewDevice, NewInstance);

	bOpen = true;

	return EAVResult::Success;
}

void FAudioEncoderWMF::Close()
{
	if (IsOpen())
	{
		Encoder = nullptr;

		bOpen = false;
	}
}

bool FAudioEncoderWMF::IsInitialized() const
{
	return IsOpen() && Encoder != nullptr;
}

FAVResult FAudioEncoderWMF::ApplyConfig()
{
	if (IsOpen())
	{
		FAudioEncoderConfigWMF const& PendingConfig = GetPendingConfig();
		if (AppliedConfig != PendingConfig)
		{
			if (!IsInitialized())
			{
				// TODO support reconfiguration

				HRESULT Result = CoCreateInstance(PendingConfig.CodecType, nullptr, CLSCTX_INPROC_SERVER, IID_IMFTransform, reinterpret_cast<void**>(&Encoder));
				if (FAILED(Result))
				{
					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to initialize encoder"), TEXT("WMF"), Result);
				}

				TRefCountPtr<IMFMediaType> InputType;
				if (FAILED(Result = MFCreateMediaType(InputType.GetInitReference()))
					|| FAILED(Result = InputType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio))
					|| FAILED(Result = InputType->SetGUID(MF_MT_SUBTYPE, MFAudioFormat_PCM))
					|| FAILED(Result = InputType->SetUINT32(MF_MT_AUDIO_BITS_PER_SAMPLE, 16))
					|| FAILED(Result = InputType->SetUINT32(MF_MT_AUDIO_SAMPLES_PER_SECOND, PendingConfig.Samplerate))
					|| FAILED(Result = InputType->SetUINT32(MF_MT_AUDIO_NUM_CHANNELS, PendingConfig.NumChannels))
					|| FAILED(Result = Encoder->SetInputType(0, InputType, 0)))
				{
					Close();

					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to configure encoder input"), TEXT("WMF"), Result);
				}

				TRefCountPtr<IMFMediaType> OutputType;
				if (FAILED(Result = MFCreateMediaType(OutputType.GetInitReference()))
					|| FAILED(Result = OutputType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio))
					|| FAILED(Result = OutputType->SetGUID(MF_MT_SUBTYPE, MFAudioFormat_AAC))
					|| FAILED(Result = OutputType->SetUINT32(MF_MT_AUDIO_BITS_PER_SAMPLE, 16))
					|| FAILED(Result = OutputType->SetUINT32(MF_MT_AUDIO_SAMPLES_PER_SECOND, PendingConfig.Samplerate))
					|| FAILED(Result = OutputType->SetUINT32(MF_MT_AUDIO_NUM_CHANNELS, PendingConfig.NumChannels))
					|| FAILED(Result = OutputType->SetUINT32(MF_MT_AUDIO_AVG_BYTES_PER_SECOND, PendingConfig.Bitrate / 8))
					|| FAILED(Result = Encoder->SetOutputType(0, OutputType, 0)))
				{
					Close();

					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to configure encoder output"), TEXT("WMF"), Result);
				}

				if (FAILED(Result = Encoder->GetInputStreamInfo(0, &InputStreamInfo))
					|| FAILED(Result = Encoder->GetOutputStreamInfo(0, &OutputStreamInfo)))
				{
					Close();

					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to retrieve encoder input/output info"), TEXT("WMF"), Result);
				}

				if (FAILED(Result = Encoder->ProcessMessage(MFT_MESSAGE_NOTIFY_BEGIN_STREAMING, 0))
					|| FAILED(Result = Encoder->ProcessMessage(MFT_MESSAGE_NOTIFY_START_OF_STREAM, 0)))
				{
					Close();

					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to start encoder stream"), TEXT("WMF"), Result);
				}
			}
		}

		return TAudioEncoder::ApplyConfig();
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("WMF"));
}

FAVResult FAudioEncoderWMF::SendFrame(TSharedPtr<FAudioResourceCPU> const& Resource, uint32 Timestamp)
{
	if (IsOpen())
	{
		FAVResult AVResult = ApplyConfig();
		if (AVResult.IsNotSuccess())
		{
			return AVResult;
		}

		Resource->Lock();

		Audio::TSampleBuffer<int16> const PCM16(
			Resource->GetRaw().Get(),
			Resource->GetNumSamples(),
			GetPendingConfig().NumChannels,
			GetPendingConfig().Samplerate);

		Resource->Unlock();

		uint32 const PCM16Size = PCM16.GetNumSamples() * sizeof(*PCM16.GetData());

		HRESULT Result = 0;

		TRefCountPtr<IMFSample> WmfSample;
		Result = MFCreateSample(WmfSample.GetInitReference());
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::ErrorMapping, TEXT("Error creating mapping sample"), TEXT("WMF"), Result);
		}

		Result = WmfSample->SetSampleTime(Timestamp * ETimespan::TicksPerSecond);
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::Error, TEXT("Error setting time of mapping sample"), TEXT("WMF"), Result);
		}

		Result = WmfSample->SetSampleDuration(Resource->GetSampleDuration() * ETimespan::TicksPerSecond);
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::Error, TEXT("Error setting duration of mapping sample"), TEXT("WMF"), Result);
		}

		TRefCountPtr<IMFMediaBuffer> WmfBuffer;
		Result = MFCreateAlignedMemoryBuffer(
			FMath::Max<int32>(InputStreamInfo.cbSize, PCM16Size),
			InputStreamInfo.cbAlignment > 1 ? InputStreamInfo.cbAlignment - 1 : 0,
			WmfBuffer.GetInitReference());
        if (FAILED(Result))
        {
        	return FAVResult(EAVResult::ErrorCreating, TEXT("Error creating input buffer"), TEXT("WMF"), Result);
        }
    
        uint8* WmfData = nullptr;
		Result = WmfBuffer->Lock(&WmfData, nullptr, nullptr);
        if (FAILED(Result))
        {
        	return FAVResult(EAVResult::ErrorLocking, TEXT("Error locking input buffer"), TEXT("WMF"), Result);
        }

		FMemory::Memcpy(WmfData, PCM16.GetData(), PCM16Size);

		WmfData = nullptr;
		Result = WmfBuffer->Unlock();
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::ErrorUnlocking, TEXT("Error unlocking input buffer"), TEXT("WMF"), Result);
		}
    
		Result = WmfBuffer->SetCurrentLength(PCM16Size);
        if (FAILED(Result))
        {
        	return FAVResult(EAVResult::Error, TEXT("Error setting size of input buffer"), TEXT("WMF"), Result);
        }

		Result = WmfSample->AddBuffer(WmfBuffer);
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::ErrorMapping, TEXT("Error adding input buffer to mapping sample"), TEXT("WMF"), Result);
		}

		Result = Encoder->ProcessInput(0, WmfSample, 0);
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::Error, TEXT("Error encoding frame"), TEXT("WMF"), Result);
		}

		Result = WmfSample->RemoveAllBuffers();
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::ErrorDestroying, TEXT("Error clearing duration of mapping sample"), TEXT("WMF"), Result);
		}

		if (OutputStreamInfo.dwFlags & MFT_OUTPUT_STREAM_PROVIDES_SAMPLES)
		{
			return FAVResult(EAVResult::Error, TEXT("WMF is unable to provide output samples"), TEXT("WMF"));
		}

		Result = MFCreateAlignedMemoryBuffer(
			OutputStreamInfo.cbSize,
			OutputStreamInfo.cbAlignment > 1 ? OutputStreamInfo.cbAlignment - 1 : 0,
			WmfBuffer.GetInitReference());
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::ErrorCreating, TEXT("Error creating output buffer"), TEXT("WMF"), Result);
		}

		Result = WmfSample->AddBuffer(WmfBuffer);
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::ErrorMapping, TEXT("Error adding output buffer to mapping sample"), TEXT("WMF"), Result);
		}

		MFT_OUTPUT_DATA_BUFFER WmfOutput = {};
        WmfOutput.pSample = WmfSample;
    
        DWORD WmfStatus = 0;
        Result = Encoder->ProcessOutput(0, 1, &WmfOutput, &WmfStatus);
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::Error, FString::Printf(TEXT("Error reading encoded frame with status %d"), WmfStatus), TEXT("WMF"), Result);
		}

		DWORD WmfBufferCount = 0;
		Result = WmfSample->GetBufferCount(&WmfBufferCount);
		if (FAILED(Result) || WmfBufferCount <= 0)
		{
			return FAVResult(EAVResult::Error, FString::Printf(TEXT("Invalid output buffer count (%d) from mapping sample"), WmfBufferCount), TEXT("WMF"), Result);
		}

		Result = WmfSample->GetBufferByIndex(0, WmfBuffer.GetInitReference());
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::Error, TEXT("Error reading output buffer from mapping sample"), TEXT("WMF"), Result);
		}

		DWORD WmfMaxLength = 0;
		DWORD WmfCurrentLength = 0;
		Result = WmfBuffer->Lock(&WmfData, &WmfMaxLength, &WmfCurrentLength);
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::ErrorLocking, TEXT("Error locking output buffer"), TEXT("WMF"), Result);
		}

		TSharedPtr<uint8> const CopiedData = MakeShareable(new uint8[WmfCurrentLength]);
		FMemory::BigBlockMemcpy(CopiedData.Get(), WmfData, WmfCurrentLength);

		WmfData = nullptr;
		Result = WmfBuffer->Unlock();
		if (FAILED(Result))
		{
			return FAVResult(EAVResult::ErrorUnlocking, TEXT("Error unlocking output buffer"), TEXT("WMF"), Result);
		}

		Packets.Enqueue(
			FAudioPacket(
				CopiedData,
				WmfCurrentLength,
				Timestamp,
				FrameCount++));

		return EAVResult::Success;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("WMF"));
}

FAVResult FAudioEncoderWMF::ReceivePacket(FAudioPacket& OutPacket)
{
	if (IsOpen())
	{
		if (Packets.Dequeue(OutPacket))
		{
			return EAVResult::Success;
		}

		return EAVResult::PendingInput;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("WMF"));
}
