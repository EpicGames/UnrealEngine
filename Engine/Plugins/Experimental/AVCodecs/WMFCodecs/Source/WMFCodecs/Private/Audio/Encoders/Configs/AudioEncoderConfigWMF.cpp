// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/Encoders/Configs/AudioEncoderConfigWMF.h"

THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include <wmcodecdsp.h>
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

#include "Audio/Encoders/Configs/AudioEncoderConfigAAC.h"

REGISTER_TYPEID(FAudioEncoderConfigWMF);

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FAudioEncoderConfigWMF& OutConfig, FAudioEncoderConfigAAC const& InConfig)
{
	OutConfig.CodecType = __uuidof(AACMFTEncoder);

	return EAVResult::Success;
}
