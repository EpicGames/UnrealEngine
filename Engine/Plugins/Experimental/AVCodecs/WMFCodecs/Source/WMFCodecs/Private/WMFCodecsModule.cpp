// Copyright Epic Games, Inc. All Rights Reserved.

#pragma comment(lib, "mfplat")
#pragma comment(lib, "mfuuid")
#pragma comment(lib, "mfreadwrite")

#include "Modules/ModuleManager.h"

#include "Audio/Encoders/AudioEncoderWMF.h"
#include "Audio/Encoders/Configs/AudioEncoderConfigAAC.h"
#include "Audio/Resources/AudioResourceCPU.h"

class FWMFCodecModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
#if PLATFORM_WINDOWS
			FAudioEncoder::RegisterPermutationsOf<FAudioEncoderWMF>
				::With<FAudioResourceCPU>
				::And<FAudioEncoderConfigWMF, FAudioEncoderConfigAAC>();
#endif
	}
};

IMPLEMENT_MODULE(FWMFCodecModule, WMFCodecs);
