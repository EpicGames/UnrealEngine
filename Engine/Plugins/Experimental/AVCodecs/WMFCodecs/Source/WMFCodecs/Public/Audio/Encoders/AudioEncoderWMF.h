// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "Containers/Queue.h"

THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include <mftransform.h>
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

#include "Audio/AudioEncoder.h"
#include "Audio/Encoders/Configs/AudioEncoderConfigWMF.h"
#include "Audio/Resources/AudioResourceCPU.h"

class WMFCODECS_API FAudioEncoderWMF : public TAudioEncoder<FAudioResourceCPU, FAudioEncoderConfigWMF>
{
private:
	TRefCountPtr<IMFTransform> Encoder = nullptr;
	MFT_INPUT_STREAM_INFO InputStreamInfo;
	MFT_OUTPUT_STREAM_INFO OutputStreamInfo;

	uint64 FrameCount = 0;

	TQueue<FAudioPacket> Packets;

public:
	uint8 bOpen : 1;
	
	FAudioEncoderWMF() = default;
	virtual ~FAudioEncoderWMF() override;

	virtual bool IsOpen() const override;
	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	virtual void Close() override;

	bool IsInitialized() const;

	virtual FAVResult ApplyConfig() override;
	
	virtual FAVResult SendFrame(TSharedPtr<FAudioResourceCPU> const& Resource, uint32 Timestamp) override;

	virtual FAVResult ReceivePacket(FAudioPacket& OutPacket) override;
};
