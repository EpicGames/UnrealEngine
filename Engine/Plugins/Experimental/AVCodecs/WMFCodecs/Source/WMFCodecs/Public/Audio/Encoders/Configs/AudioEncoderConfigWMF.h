// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include <guiddef.h>
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

#include "Audio/AudioEncoder.h"

struct WMFCODECS_API FAudioEncoderConfigWMF : public FAudioEncoderConfig
{
public:
	GUID CodecType = {};

	FAudioEncoderConfigWMF(EAVPreset Preset = EAVPreset::Default)
		: FAudioEncoderConfig(Preset)
	{
	}

	bool operator==(FAudioEncoderConfigWMF const& Other) const
	{
		return CodecType == Other.CodecType;
	}

	bool operator!=(FAudioEncoderConfigWMF const& Other) const
	{
		return !(*this == Other);
	}
};

template <>
FAVResult FAVExtension::TransformConfig(FAudioEncoderConfigWMF& OutConfig, struct FAudioEncoderConfigAAC const& InConfig);

DECLARE_TYPEID(FAudioEncoderConfigWMF, WMFCODECS_API);
