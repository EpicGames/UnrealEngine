// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

public class WMFCodecs : ModuleRules
{
	public WMFCodecs(ReadOnlyTargetRules Target) : base(Target)
	{
		bLegacyPublicIncludePaths = false;
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Engine",
			"AVCodecsCore",
		});

		PublicDependencyModuleNames.AddRange(new string[] {
			"RenderCore",
			"Core",
		});
		
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			PublicDelayLoadDLLs.Add("mfplat.dll");
			PublicDelayLoadDLLs.Add("mfuuid.dll");
			PublicDelayLoadDLLs.Add("mfreadwrite.dll");
		}
	}
}
