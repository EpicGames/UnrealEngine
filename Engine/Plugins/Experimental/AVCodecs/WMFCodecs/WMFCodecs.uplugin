{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "WMFCodecs",
	"Description": "Adds codecs from the Windows Media Foundation to AVCodecs",
	"Category": "Codecs",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsExperimentalVersion": true,
	"Modules": [
		{
			"Name": "WMFCodecs",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"TargetDenyList": [
				"Server"
			],
			"PlatformAllowList": [
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "AVCodecsCore",
			"Enabled": true
		}
	],
	"SupportedTargetPlatforms": [
		"Win64"
	]
}
